{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "AIModuleToolset",
	"Description": "Toolset for AIModule Systems",
	"Category": "AI",
	"EditorOnly": true,
	"CreatedBy": "Epic Games",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"NoRedist": true,
	"EnabledByDefault": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "AIModuleToolset",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		}
	]
}
