// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AIModuleToolset : ModuleRules
{
	public AIModuleToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		PublicDependencyModuleNames.Add("Core");
	}
}
