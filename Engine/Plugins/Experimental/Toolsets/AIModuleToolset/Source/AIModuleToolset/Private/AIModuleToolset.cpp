// Copyright Epic Games, Inc. All Rights Reserved.

#include "AIModuleToolset.h"

#include "Modules/ModuleManager.h"

/*virtual*/ void FAIModuleToolsetModule::StartupModule()
{
}

/*virtual*/ void FAIModuleToolsetModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FAIModuleToolsetModule, AIModuleToolset);
DEFINE_LOG_CATEGORY(LogAIModuleToolset);
