// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAIModuleToolset, Log, Log);

class FAIModuleToolsetModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
